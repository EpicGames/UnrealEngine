// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
    public class Chaos_Windows : Chaos
    {
        public Chaos_Windows(ReadOnlyTargetRules Target) : base(Target)
        {
			// Disabling the optimized VectorGather call because it is causing crashes on PC.
			PublicDefinitions.Add("USE_ISPC_OPTIMIZED_VECTORGATHER=0");
			// Disabling the optimized VectorScatter call because it is causing crashes on PC.
			PublicDefinitions.Add("USE_ISPC_OPTIMIZED_VECTORSCATTER=0");
		}
    }
}
